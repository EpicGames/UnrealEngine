{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Post Process Material Chain Graph",
	"Description": "Post Process Material Chain Graph allows users to stack post process materials and render those into render targets separate from Scene Color.\nThis can operate on textures other than scene color without writing those into scene color.",
	"Category": "Other",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PPMChainGraph",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "PPMChainGraphEditor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}

	]
}