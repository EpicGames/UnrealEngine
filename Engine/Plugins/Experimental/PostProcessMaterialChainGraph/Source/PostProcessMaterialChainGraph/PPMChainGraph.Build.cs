// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class PPMChainGraph : ModuleRules
{
	public PPMChainGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
				
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RHI",
				"Projects",
				"Renderer",
				"RenderCore",
				"Slate",
				"SlateCore",

			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
