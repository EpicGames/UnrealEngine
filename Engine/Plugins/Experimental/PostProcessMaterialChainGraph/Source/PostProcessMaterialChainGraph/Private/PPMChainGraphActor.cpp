// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphActor.h"
#include "PPMChainGraphComponent.h"
#include "Components/BillboardComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"

APPMChainGraphActor::APPMChainGraphActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));

	PPMChainGraphExecutorComponent = CreateDefaultSubobject<UPPMChainGraphExecutorComponent>(TEXT("PPMChainGraphExecutorComponent0"));


#if WITH_METADATA
	{
		// Create billboard component
		if (GIsEditor && !IsRunningCommandlet())
		{
			// Structure to hold one-time initialization

			struct FConstructorStatics
			{
				ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
				FName ID_PPMChainGraphActor;
				FText NAME_PPMChainGraphActor;

				FConstructorStatics()
					: SpriteTextureObject(TEXT("/Engine/EditorResources/S_Actor"))
					, ID_PPMChainGraphActor(TEXT("PPM Chain Graph Actor"))
					, NAME_PPMChainGraphActor(NSLOCTEXT("SpriteCategory", "PPMChainGraphActor", "PPM Chain Graph Actor"))
				{
				}
			};

			static FConstructorStatics ConstructorStatics;

			SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("PPM Chain Graph Component Icon"));

			if (SpriteComponent)
			{
				SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
				SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_PPMChainGraphActor;
				SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_PPMChainGraphActor;
				SpriteComponent->SetIsVisualizationComponent(true);
				SpriteComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
				SpriteComponent->SetMobility(EComponentMobility::Movable);
				SpriteComponent->bHiddenInGame = true;
				SpriteComponent->bIsScreenSizeScaled = true;

				SpriteComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
			}
		}

	}
#endif 
}
