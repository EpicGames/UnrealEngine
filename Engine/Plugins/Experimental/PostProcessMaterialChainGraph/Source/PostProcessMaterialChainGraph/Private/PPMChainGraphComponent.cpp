// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphComponent.h"
#include "Camera/CameraActor.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Framework/Notifications/NotificationManager.h"
#include "PPMChainGraph.h"
#include "PPMChainGraphWorldSubsystem.h"
#include "RenderingThread.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "PPMChainGraphComponent"
DEFINE_LOG_CATEGORY(LogPPMChainGraph);

#define CAMERA_INVALID_PARENT_WARNING "Post Process Material Chain Graph Executor Component requires to be parented to Camera Actor. This component {0} on Actor \"{1}\" will be disabled until it is parented to Camera Actor."
#define IS_STREAMED_TEXTURE_WARNING "The following external texture is streamed. {0} . \nThis may cause unexpected results. It is suggested to set NeverStream in texture properties and making sure Virtual Texture streaming is disabled."
#define INVALID_MATERIAL_DOMAIN "The following material's domain isn't set to Post Processing. {0} . \nPost Process Material Chain Graph only supports Post Process Materials."
#define RENDER_TARGET_ID_IS_IN_USE "Render target name is already in use. {0} . \nThe pass will not be executed until a valid name is provided."
#define INPUT_IS_USED_AS_RENDERTARGET "Input is used as a render target. {0} . \nUsing the same input as a render target in the same pass is not allowed. Pass will not be executed."

namespace
{
	struct FPPMChainGraphAggregatedWarnings
	{
		struct FWarnings
		{
			TSet<FString> TextureRequiresStreamingWarningList;
			TSet<FString> InvalidMaterialDomainWarningList;
			TSet<FString> TextureIdIsInUseWarningList;
			TSet<FString> InputIsUsedAsAnOutputWarningList;
		};
		FWarnings CurrentFrame;
		FWarnings LastFrame;
	};
}

UPPMChainGraphExecutorComponent::UPPMChainGraphExecutorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AggregatedWarnings(MakeShared<FPPMChainGraphAggregatedWarnings>())
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
	bAutoActivate = true;
}

void UPPMChainGraphExecutorComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	{
		AActor* OwnerActor = GetOwner();
		if (!IsValid(OwnerActor))
		{
			return;
		}

		bool bIsActiveLoc = IsValidChecked(this)
#if WITH_EDITOR
			&& !GetOwner()->IsHiddenEd()
#endif 
			&& !GetOwner()->IsHidden();

		if (!bIsActiveLoc)
		{
			FScopeLock ScopeLock(&StateTransferCriticalSection);
			PPMChainGraphsRenderProxies.Empty();
			return;
		}
	}

	if (const UWorld* World = GetWorld())
	{
		if (!PPMChainGraphSubsystem.IsValid())
		{
			PPMChainGraphSubsystem = World->GetSubsystem<UPPMChainGraphWorldSubsystem>();
		}

		// We attempt to add every frame and remove it only on destroy to avoid complications in handling cases such as 
		// undo/redo and other transactions. This way this object becomes invalid only when it is no longer ticking.
		if (PPMChainGraphSubsystem.IsValid())
		{
			PPMChainGraphSubsystem->AddPPMChainGraphComponent(this);
		}
	}

	// Validating graphs and preparing them for the use on render thread.
	TransferState();

	// Display aggregated warnings.
	ProcessWarnings();
};

void UPPMChainGraphExecutorComponent::BeginDestroy()
{
	if (PPMChainGraphSubsystem.IsValid())
	{
		PPMChainGraphSubsystem->RemovePPMChainGraphComponent(this);
	}

	Super::BeginDestroy();
}

void UPPMChainGraphExecutorComponent::TransferState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("PPMChainGraph.TransferState %s"), *GetName()));
	TMap<EPPMChainGraphExecutionLocation, TArray<TSharedPtr<FPPMChainGraphProxy>>> TempChainGraphRenderProxies;
	FPPMChainGraphAggregatedWarnings* AggregatedWarningsCast = static_cast<FPPMChainGraphAggregatedWarnings*>(AggregatedWarnings.Get());

	for (TWeakObjectPtr<UPPMChainGraph> Graph : PPMChainGraphs)
	{
		if (!Graph.IsValid())
		{
			continue;
		}

		// Ignore the graph if there are no passes.
		if (Graph->Passes.IsEmpty())
		{
			continue;
		}
		TSharedPtr<FPPMChainGraphProxy> GraphRenderProxy = MakeShared<FPPMChainGraphProxy>();
		GraphRenderProxy->PointOfExecution = Graph->PointOfExecution;
		TArray<TSharedPtr<FPPMChainGraphProxy>>& ProxiesPerPointOfExecution = TempChainGraphRenderProxies.FindOrAdd(Graph->PointOfExecution);
		ProxiesPerPointOfExecution.Add(GraphRenderProxy);

		// Used to make sure that we don't have duplicate ids. 
		TSet<FString> TextureIds;
		for (TTuple<FString, TObjectPtr<UTexture2D>> KeyValue : Graph->ExternalTextures)
		{
			if (KeyValue.Key.IsEmpty() || !IsValid(KeyValue.Value))
			{
				continue;
			}
			TextureIds.Add(KeyValue.Key);
			GraphRenderProxy->ExternalTextures.Add(KeyValue);

			if (!KeyValue.Value->IsFullyStreamedIn() || KeyValue.Value->VirtualTextureStreaming)
			{
				FString TextureKey = FString::Format(TEXT("{0}, {1}"), { *KeyValue.Value.GetFullName(), *Graph->GetFullName()});
				AggregatedWarningsCast->CurrentFrame.TextureRequiresStreamingWarningList.FindOrAdd(TextureKey);
			}
		}

		// Validate each pass and create proxies.
		for (const FPPMChainGraphPostProcessPass& Pass : Graph->Passes)
		{
			if (!Pass.bEnabled)
			{
				continue;
			}
			if (!Pass.PostProcessMaterial)
			{
				continue;
			}

			// Queue a warning about Post process material being invalid.
			if (Pass.PostProcessMaterial->MaterialDomain != EMaterialDomain::MD_PostProcess)
			{
				FString MaterialIdString = FString::Format(TEXT("{0}, {1}"), { *Pass.PostProcessMaterial.GetFullName(), *Graph->GetFullName()});
				AggregatedWarningsCast->CurrentFrame.InvalidMaterialDomainWarningList.FindOrAdd(MaterialIdString);
				continue;
			}

			// Queue a warning about Render target Id already being used.
			if (TextureIds.Contains(Pass.TemporaryRenderTargetId))
			{
				FString TextureId = FString::Format(TEXT("Render Target Name: {0}, {1}"), { *Pass.TemporaryRenderTargetId, *Graph->GetFullName() });
				AggregatedWarningsCast->CurrentFrame.TextureIdIsInUseWarningList.FindOrAdd(TextureId);
				continue;
			}

			// Ignore empty ids. 
			if (Pass.Output == EPPMChainGraphOutput::PPMOutput_RenderTarget && Pass.TemporaryRenderTargetId.IsEmpty())
			{
				continue;
			}

			TextureIds.Add(Pass.TemporaryRenderTargetId);

			TSharedPtr<FPPMChainGraphPostProcessPass> PassRenderProxy = MakeShared<FPPMChainGraphPostProcessPass>();
			PassRenderProxy->PostProcessMaterial = Pass.PostProcessMaterial;
			PassRenderProxy->TemporaryRenderTargetId = Pass.TemporaryRenderTargetId;
			PassRenderProxy->Output = Pass.Output;

			// We cannot write into the same render target that is used as input.
			bool bInputReferencesOutput = false;

			// Validate inputs.
			for (TTuple<EPPMChainGraphPPMInputId, FPPMChainGraphInput> KeyValue : Pass.Inputs)
			{
				if (KeyValue.Value.InputId == Pass.TemporaryRenderTargetId)
				{
					bInputReferencesOutput = true;

					// Queue a warning about input being used as a render target.
					FString TextureId = FString::Format(TEXT("{0}, {1}"), { *Pass.TemporaryRenderTargetId, *Graph->GetFullName() });
					AggregatedWarningsCast->CurrentFrame.InputIsUsedAsAnOutputWarningList.FindOrAdd(TextureId);
					break;
				}
				PassRenderProxy->Inputs.Add(KeyValue);
			}

			// Pass is invalid due to output writing into a texture that is used as an input in the same pass.
			if (bInputReferencesOutput)
			{
				continue;
			}

			GraphRenderProxy->Passes.Add(MoveTemp(PassRenderProxy));
		}
	}

	// Transfer the state on render thread, which is where it is going to be consumed.
	{
		FScopeLock ScopeLock(&StateTransferCriticalSection);
		PPMChainGraphsRenderProxies = TempChainGraphRenderProxies;
	};
}

void UPPMChainGraphExecutorComponent::ProcessWarnings()
{
	FPPMChainGraphAggregatedWarnings* AggregatedWarningsCast = static_cast<FPPMChainGraphAggregatedWarnings*>(AggregatedWarnings.Get());
	auto ShowToastFunc = [&](const FString& InText)
	{
		UE_LOGF(LogPPMChainGraph, Warning, "%ls", *InText);

#if WITH_EDITOR
		FNotificationInfo Info(FText::FromString(InText));
		Info.ExpireDuration = 5.0f;

		FSlateNotificationManager::Get().AddNotification(Info);
#endif
	};
	
	auto DisplayWarning = [ShowToastFunc](const TSet<FString>& InLastFrameNotifications, const TSet<FString>& InCurrentFrameNotifications, const FString& InMessageFormat)
	{
		if (!InCurrentFrameNotifications.IsEmpty())
		{
			TSet<FString> DiffSet = InCurrentFrameNotifications.Difference(InLastFrameNotifications);
			for (const FString& Notification : DiffSet)
			{
				if (InCurrentFrameNotifications.Contains(Notification))
				{
					ShowToastFunc(FString::Format(*InMessageFormat, {Notification}));
				}
			}
		}
	};

	// Streaming texture warnings.
	{
		DisplayWarning(AggregatedWarningsCast->LastFrame.TextureRequiresStreamingWarningList
			, AggregatedWarningsCast->CurrentFrame.TextureRequiresStreamingWarningList
			, IS_STREAMED_TEXTURE_WARNING
		);
	}

	// Invalid Material Domain warnings.
	{
		DisplayWarning(AggregatedWarningsCast->LastFrame.InvalidMaterialDomainWarningList
			, AggregatedWarningsCast->CurrentFrame.InvalidMaterialDomainWarningList
			, INVALID_MATERIAL_DOMAIN
		);
	}

	// Render target Id is already in use warning.
	{
		DisplayWarning(AggregatedWarningsCast->LastFrame.TextureIdIsInUseWarningList
			, AggregatedWarningsCast->CurrentFrame.TextureIdIsInUseWarningList
			, RENDER_TARGET_ID_IS_IN_USE
		);
	}

	// Render target Id is already in use warning.
	{
		DisplayWarning(AggregatedWarningsCast->LastFrame.InputIsUsedAsAnOutputWarningList
			, AggregatedWarningsCast->CurrentFrame.InputIsUsedAsAnOutputWarningList
			, INPUT_IS_USED_AS_RENDERTARGET
		);
	}
	AggregatedWarningsCast->LastFrame = MoveTemp(AggregatedWarningsCast->CurrentFrame);
	AggregatedWarningsCast->CurrentFrame = FPPMChainGraphAggregatedWarnings::FWarnings();

}

TArray<TSharedPtr<FPPMChainGraphProxy>> UPPMChainGraphExecutorComponent::GetChainGraphRenderProxies(EPPMChainGraphExecutionLocation InPointOfExecution)
{
	FScopeLock ScopeLock(&StateTransferCriticalSection);
	if (PPMChainGraphsRenderProxies.Contains(InPointOfExecution))
	{
		return PPMChainGraphsRenderProxies[InPointOfExecution];
	}
	else
	{
		return TArray<TSharedPtr<FPPMChainGraphProxy>>();
	}
}

bool UPPMChainGraphExecutorComponent::IsActiveDuringPass_GameThread(EPPMChainGraphExecutionLocation InPointOfExecution)
{
	check(IsInGameThread());
	return PPMChainGraphsRenderProxies.Contains(InPointOfExecution);
}

#undef LOCTEXT_NAMESPACE 