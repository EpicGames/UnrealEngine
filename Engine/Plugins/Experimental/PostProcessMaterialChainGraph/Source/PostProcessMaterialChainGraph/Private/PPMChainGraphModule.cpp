// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphModule.h"

void FPostProcessMaterialChainGraphModule::StartupModule()
{
}

void FPostProcessMaterialChainGraphModule::ShutdownModule()
{
}
	
IMPLEMENT_MODULE(FPostProcessMaterialChainGraphModule, PPMChainGraph)