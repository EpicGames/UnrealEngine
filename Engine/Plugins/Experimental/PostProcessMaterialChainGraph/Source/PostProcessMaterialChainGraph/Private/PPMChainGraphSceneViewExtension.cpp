// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphSceneViewExtension.h"
#include "Camera/CameraActor.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DynamicResolutionState.h"
#include "FXRenderingUtils.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PostProcess/DrawRectangle.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "PostProcess/PostProcessInputs.h"
#include "PPMChainGraphComponent.h"
#include "PPMChainGraphWorldSubsystem.h"
#include "PPMChainGraph.h"
#include "RHI.h"
#include "ScreenPass.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"

DECLARE_GPU_STAT_NAMED(PPMChain, TEXT("PPMChainRender"));

namespace
{
	void RenderPPMChainGraphProxies
		( FRDGBuilder& GraphBuilder
		, const FSceneView& View
		, const FSceneViewFamily& ViewFamily
		, const TArray<TSharedPtr<FPPMChainGraphProxy>>& ChainGraphProxy
		, const FIntRect& PrimaryViewRect
		, const FScreenPassRenderTarget& SceneColorRenderTarget
		, FScreenPassRenderTarget& BackBufferRenderTarget
		, FPostProcessMaterialInputs& PostProcessMaterialInputs
		, EPPMChainGraphExecutionLocation PointOfExecution)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, PPMChain, "PPMChainGraph.Render");

		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
		
		const FScreenPassTextureViewport RegionViewport(SceneColorRenderTarget.Texture, PrimaryViewRect);

		FScreenPassRenderTarget MergedStencilRenderTarget;

		bool bPPMApplied = false;

		FRDGTextureDesc OutputDesc = SceneColorRenderTarget.Texture->Desc;
		FLinearColor ClearColor(0., 0., 1., 0.);
		OutputDesc.ClearValue = FClearValueBinding(ClearColor);
		OutputDesc.Flags |= ETextureCreateFlags::RenderTargetable;

		FScreenPassRenderTarget CopyBackTexture;
		bool bCopyBackTexture = false;

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		for (const TSharedPtr<FPPMChainGraphProxy>& Graph : ChainGraphProxy)
		{
			if (!Graph.IsValid())
			{
				continue;
			}

			TMap<FString, FScreenPassRenderTarget> PassOutputs;
			int32 PassId = 0;
			for (const TSharedPtr<FPPMChainGraphPostProcessPass>& Pass : Graph->Passes)
			{
				if (!Pass->PostProcessMaterial || Pass->PostProcessMaterial->MaterialDomain != EMaterialDomain::MD_PostProcess)
				{
					continue;
				}

				bPPMApplied = true;

				const FScreenPassTextureViewport OutputViewport(SceneColorRenderTarget);

				if (Pass->Output == EPPMChainGraphOutput::PPMOutput_RenderTarget)
				{
					// Makes sure that the same render target won't be used as an input in the same pass. 
					// There is an additional validation on Game thread.
					if (PassOutputs.Contains(Pass->TemporaryRenderTargetId))
					{
						continue;
					}
					FRDGTexture* PassRDGTexture = GraphBuilder.CreateTexture(OutputDesc, *Pass->TemporaryRenderTargetId);
					FScreenPassRenderTarget PassRenderTarget = FScreenPassRenderTarget(PassRDGTexture, SceneColorRenderTarget.ViewRect, ERenderTargetLoadAction::EClear);
					PassOutputs.Add(Pass->TemporaryRenderTargetId, PassRenderTarget);

					PostProcessMaterialInputs.OverrideOutput = PassRenderTarget;
					CopyBackTexture = PassRenderTarget;
				}
				else
				{

					PostProcessMaterialInputs.OverrideOutput = BackBufferRenderTarget;
					CopyBackTexture = BackBufferRenderTarget;
				}

				bCopyBackTexture = true;

				TStaticArray<FScreenPassTextureSlice, kPostProcessMaterialInputCountMax> PPMInputTextures;

				// Mapping our custom inputs to PPM Inputs.
				for (uint32 InputIndex = 0; InputIndex < kPostProcessMaterialInputCountMax; ++InputIndex)
				{
					bool bUseDefaultInput = true;
					EPPMChainGraphPPMInputId InputId = (EPPMChainGraphPPMInputId)(InputIndex + 1);
					if (Pass->Inputs.Contains(InputId))
					{
						FString InputMappedToId = Pass->Inputs[InputId].InputId;
						// Need to exclude current pass.
						if (PassOutputs.Contains(InputMappedToId))
						{
							PPMInputTextures[InputIndex] = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutputs[InputMappedToId]);
							bUseDefaultInput = false;
						}
						else if (Graph->ExternalTextures.Contains(InputMappedToId))
						{
							TWeakObjectPtr<UTexture2D> ExternalTexture = Graph->ExternalTextures[InputMappedToId];

							if (ExternalTexture.IsValid())
							{
								FRHITexture* ExternalTextureResourceRHI = ExternalTexture->GetResource() ? ExternalTexture->GetResource()->GetTexture2DRHI() : nullptr;
								bool bIsShaderResource = false;

								if (ExternalTextureResourceRHI)
								{
									const FRHITextureDesc& ExternalTexDesc = ExternalTextureResourceRHI->GetDesc();

									// Virtual textures are not used as shader resources and users are notified about it.
									bIsShaderResource = EnumHasAnyFlags(ExternalTexDesc.Flags, ETextureCreateFlags::ShaderResource);
								}

								if (ExternalTextureResourceRHI && bIsShaderResource)
								{
									ExternalTextureResourceRHI->GetDesc();
									FRDGTextureRef ExternalTextureRDG = RegisterExternalTexture(GraphBuilder, ExternalTextureResourceRHI, *InputMappedToId);
									FScreenPassTexture ExternalTextureInput(ExternalTextureRDG, FIntRect(0, 0, ExternalTexture->GetSizeX(), ExternalTexture->GetSizeY()));
									PPMInputTextures[InputIndex] = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ExternalTextureInput);

									bUseDefaultInput = false;
								}
							}
						}
					}
					
					if (bUseDefaultInput)
					{
						FScreenPassTextureSlice Input = PostProcessMaterialInputs.GetInput((EPostProcessMaterialInput)InputIndex);
						PPMInputTextures[InputIndex] = Input;
					}
				}

				PostProcessMaterialInputs.Textures = MoveTemp(PPMInputTextures);
				AddPostProcessMaterialPass(GraphBuilder, View, PostProcessMaterialInputs, Pass->PostProcessMaterial);
			}
		}

		// PrePostProcess is the only place where we need to copy back into Scene Color. In all other places we return back the backbuffer.
		if (bCopyBackTexture && PointOfExecution == EPPMChainGraphExecutionLocation::PrePostProcess)
		{
			// Since we've rendered into the backbuffer already we have to use load flag instead.
			CopyBackTexture.LoadAction = ERenderTargetLoadAction::ELoad;

			TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);
			TShaderMapRef<FCopyRectPS> PixelShader(GlobalShaderMap);

			FCopyRectPS::FParameters* PixelShaderParameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
			PixelShaderParameters->InputTexture = CopyBackTexture.Texture;
			PixelShaderParameters->InputSampler = TStaticSamplerState<>::GetRHI();
			PixelShaderParameters->RenderTargets[0] = SceneColorRenderTarget.GetRenderTargetBinding();

			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("PPMChain_CopyViewport"),
				View,
				RegionViewport,
				RegionViewport,
				VertexShader,
				PixelShader,
				PixelShaderParameters
			);
		}
	}
}

FPPMChainGraphSceneViewExtension::FPPMChainGraphSceneViewExtension(const FAutoRegister& AutoRegister, UPPMChainGraphWorldSubsystem* InWorldSubsystem)
	: FSceneViewExtensionBase(AutoRegister)
	, WorldSubsystem(InWorldSubsystem)
{

}

void FPPMChainGraphSceneViewExtension::GatherChainGraphProxies
(TArray<TSharedPtr<FPPMChainGraphProxy>>& OutChainGraphProxies
	, const FSceneView& InView
	, const FSceneViewFamily& InViewFamily
	, EPPMChainGraphExecutionLocation InPointOfExecution)
{
	TSet<TWeakObjectPtr<UPPMChainGraphExecutorComponent>> PPMChainGraphComponents;
	if (!WorldSubsystem.IsValid())
	{
		return;
	}

	{
		FScopeLock ScopeLock(&WorldSubsystem->ComponentAccessCriticalSection);
		PPMChainGraphComponents = WorldSubsystem->PPMChainGraphComponents;
	}
	for (TWeakObjectPtr<UPPMChainGraphExecutorComponent> PPMChainGraphComponent : PPMChainGraphComponents)
	{
		if (!PPMChainGraphComponent.IsValid())
		{
			continue;
		}
		if (PPMChainGraphComponent->PPMChainGraphs.Num() == 0)
		{
			continue;
		}
		if (
			PPMChainGraphComponent->IsBeingDestroyed() ||
			PPMChainGraphComponent->GetWorld() != InViewFamily.Scene->GetWorld()
			)
		{
			continue;
		}

		if (PPMChainGraphComponent->CameraList.IsEmpty() && PPMChainGraphComponent->CameraViewHandlingMode == ECameraViewHandling::RenderOnlyInSelectedCameraViews)
		{
			continue;
		}

		bool bCameraOptionIsValid = PPMChainGraphComponent->CameraViewHandlingMode == ECameraViewHandling::IgnoreCameraViews ? true : false;
		for (const TSoftObjectPtr<ACameraActor>& Camera : PPMChainGraphComponent->CameraList)
		{
			if (!Camera.IsValid())
			{
				continue;
			}

			if (PPMChainGraphComponent->CameraViewHandlingMode == ECameraViewHandling::IgnoreCameraViews)
			{
				if (Camera.Get() == InView.ViewActor)
				{
					bCameraOptionIsValid = false;
					break;
				}
			}
			else
			{
				if (Camera.Get() == InView.ViewActor)
				{
					bCameraOptionIsValid = true;
					break;
				}
			}
		}
		if (!bCameraOptionIsValid)
		{
			continue;
		}
		OutChainGraphProxies.Append(PPMChainGraphComponent->GetChainGraphRenderProxies(InPointOfExecution));
	}
}

void FPPMChainGraphSceneViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	{
		FScopeLock ScopeLock(&WorldSubsystem->ActiveAccessCriticalSection);
		if (!WorldSubsystem->ActivePasses.Contains((uint32)PassId + 1))
		{
			return;
		}
	}

	TArray<TSharedPtr<FPPMChainGraphProxy>> ChainGraphProxies;
	EPPMChainGraphExecutionLocation PointOfExecution = (EPPMChainGraphExecutionLocation)((int)PassId + 1);
	GatherChainGraphProxies(ChainGraphProxies, View, *View.Family, PointOfExecution);

	if (!ChainGraphProxies.IsEmpty())
	{
		InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateLambda([this, InPassId = PassId, ChainGraphProxies = MoveTemp(ChainGraphProxies)](FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InInputs)
			{
				FPostProcessMaterialInputs InOutInputs = InInputs;
				return FPPMChainGraphSceneViewExtension::AfterPostProcessPass_RenderThread(GraphBuilder, View, InOutInputs, InPassId, ChainGraphProxies);
			}));
	}
}

FScreenPassTexture FPPMChainGraphSceneViewExtension::AfterPostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FPostProcessMaterialInputs& InOutInputs, EPostProcessingPass InCurrentPass, const TArray<TSharedPtr<FPPMChainGraphProxy>>& InChainGraphProxy)
{
	const FSceneViewFamily& ViewFamily = *View.Family;

	TArray<TSharedPtr<FPPMChainGraphProxy>> ChainGraphProxies;

	EPPMChainGraphExecutionLocation PointOfExecution = (EPPMChainGraphExecutionLocation)((int)InCurrentPass + 1);
	FScreenPassRenderTarget OverrideOutput = InOutInputs.OverrideOutput;

	const FIntRect PrimaryViewRect = UE::FXRenderingUtils::GetRawViewRectUnsafe(View);
	const FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));

	check(SceneColor.IsValid());

	// Reusing the same output description for our back buffer as SceneColor
	FRDGTextureDesc OutputDesc = SceneColor.Texture->Desc;
	OutputDesc.Format = PF_FloatRGBA;
	FLinearColor ClearColor(0., 0., 0., 0.);
	OutputDesc.ClearValue = FClearValueBinding(ClearColor);
	OutputDesc.Flags |= ETextureCreateFlags::RenderTargetable;

	FScreenPassRenderTarget BackBufferRenderTarget = InOutInputs.OverrideOutput;

	// If the override output is provided, it means that this is the last pass in post processing.
	if (!BackBufferRenderTarget.IsValid())
	{
		FRDGTexture* BackBufferRenderTargetTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BackBufferRenderTargetTexture"));
		BackBufferRenderTarget = FScreenPassRenderTarget(BackBufferRenderTargetTexture, SceneColor.ViewRect, ERenderTargetLoadAction::EClear);
	}

	FScreenPassRenderTarget SceneColorRenderTarget(SceneColor, ERenderTargetLoadAction::ELoad);

	RenderPPMChainGraphProxies
	(
		GraphBuilder
		, View
		, ViewFamily
		, ChainGraphProxies
		, PrimaryViewRect
		, SceneColorRenderTarget
		, BackBufferRenderTarget
		, InOutInputs
		, PointOfExecution
	);

	FRDGViewableResource* Resource = BackBufferRenderTarget.Texture;
	if (Resource->HasBeenProduced())
	{
		return MoveTemp(BackBufferRenderTarget);
	}
	else
	{
		InOutInputs.OverrideOutput = OverrideOutput;
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
}

void FPPMChainGraphSceneViewExtension::PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessingInputs& Inputs)
{
	CachedView = &View;
	const FSceneViewFamily& ViewFamily = *View.Family;

	TArray<TSharedPtr<FPPMChainGraphProxy>> ChainGraphProxies;
	GatherChainGraphProxies(ChainGraphProxies, View, ViewFamily, EPPMChainGraphExecutionLocation::PrePostProcess);

	if (ChainGraphProxies.IsEmpty())
	{
		// Don't need to modify anything, just return the untouched scene color texture back to post processing.
		return;
	}

	const FIntRect PrimaryViewRect = UE::FXRenderingUtils::GetRawViewRectUnsafe(View);
	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, PrimaryViewRect);
	check(SceneColor.IsValid());

	// Reusing the same output description for our back buffer as SceneColor
	FRDGTextureDesc OutputDesc = SceneColor.Texture->Desc;

	OutputDesc.Format = PF_FloatRGBA;
	FLinearColor ClearColor(0., 0., 0., 0.);
	OutputDesc.ClearValue = FClearValueBinding(ClearColor);
	OutputDesc.Flags |= ETextureCreateFlags::RenderTargetable;

	FScreenPassRenderTarget SceneColorRenderTarget(SceneColor, ERenderTargetLoadAction::ELoad);
	FRDGTexture* BackBufferRenderTargetTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BackBufferRenderTargetTexture"));
	FScreenPassRenderTarget BackBufferRenderTarget = FScreenPassRenderTarget(BackBufferRenderTargetTexture, SceneColor.ViewRect, ERenderTargetLoadAction::EClear);

	const auto GetPostProcessMaterialInputs = [&](FScreenPassTexture InSceneColor)
	{
		FPostProcessMaterialInputs PostProcessMaterialInputs;

		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, InSceneColor);
		const FScreenPassTexture Velocity((*Inputs.SceneTextures)->GBufferVelocityTexture, PrimaryViewRect);
		FTranslucencyPassResources PostDOFTranslucencyResources = Inputs.TranslucencyViewResourcesMap.Get(ETranslucencyPass::TPT_TranslucencyAfterDOF);

		FIntRect ViewRect{ 0, 0, 1, 1 };

		if (Inputs.PathTracingResources.bPostProcessEnabled)
		{
			const FPathTracingResources& PathTracingResources = Inputs.PathTracingResources;

			ViewRect = InSceneColor.ViewRect;
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Radiance, FScreenPassTexture(PathTracingResources.Radiance, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::DenoisedRadiance, FScreenPassTexture(PathTracingResources.DenoisedRadiance, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Albedo, FScreenPassTexture(PathTracingResources.Albedo, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Normal, FScreenPassTexture(PathTracingResources.Normal, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Variance, FScreenPassTexture(PathTracingResources.Variance, ViewRect));
		}

		if (PostDOFTranslucencyResources.IsValid())
		{
			ViewRect = PostDOFTranslucencyResources.ViewRect;
		}

		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SeparateTranslucency, FScreenPassTexture(PostDOFTranslucencyResources.GetColorForRead(GraphBuilder), ViewRect));
		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::Velocity, Velocity);
		PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
		const FScreenPassTexture CustomDepth(Inputs.CustomDepthTexture, PrimaryViewRect);
		PostProcessMaterialInputs.CustomDepthTexture = CustomDepth.Texture;
		PostProcessMaterialInputs.bManualStencilTest = Inputs.bSeparateCustomStencil;
		// This is not available to Scene View Extensions yet.
		//PostProcessMaterialInputs.SceneWithoutWaterTextures = &SceneWithoutWaterTextures;

		return PostProcessMaterialInputs;
	};

	FPostProcessMaterialInputs PPMInputs = GetPostProcessMaterialInputs(SceneColor);

	RenderPPMChainGraphProxies
	(
		GraphBuilder
		, View
		, ViewFamily
		, ChainGraphProxies
		, PrimaryViewRect
		, SceneColorRenderTarget
		, BackBufferRenderTarget
		, PPMInputs
		, EPPMChainGraphExecutionLocation::PrePostProcess
	);
}
