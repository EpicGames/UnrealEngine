// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SceneViewExtension.h"
#include "PPMChainGraph.h"
class UPPMChainGraphWorldSubsystem;
struct FPPMChainGraphProxy;

/**
* Scene View Extension responsible for going through all PPM Chain Graph Components and rendering graphs into scene color.
*/
class FPPMChainGraphSceneViewExtension : public FSceneViewExtensionBase
{
public:
	FPPMChainGraphSceneViewExtension(const FAutoRegister& AutoRegister, UPPMChainGraphWorldSubsystem* InWorldSubsystem);

	//~ Begin FSceneViewExtensionBase Interface
	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;

	virtual void PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessingInputs& Inputs) override;
	FScreenPassTexture AfterPostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FPostProcessMaterialInputs& InOutInputs, EPostProcessingPass InCurrentPass, const TArray<TSharedPtr<FPPMChainGraphProxy>>& OutChainGraphProxies);

	virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override
	{
		CachedView = nullptr;
	}

private:
	void GatherChainGraphProxies(TArray<TSharedPtr<FPPMChainGraphProxy>>& OutChainGraphProxies, const FSceneView& InView, const FSceneViewFamily& InViewFamily, EPPMChainGraphExecutionLocation InPointOfExecution);

private:
	const FSceneView* CachedView = nullptr;
	TWeakObjectPtr<UPPMChainGraphWorldSubsystem> WorldSubsystem;
};