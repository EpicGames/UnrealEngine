// Copyright Epic Games, Inc. All Rights Reserved.
#include "PPMChainGraphWorldSubsystem.h"
#include "PPMChainGraphSceneViewExtension.h"
#include "PPMChainGraphComponent.h"

#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "EngineUtils.h"
#include "SceneViewExtension.h"


#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "PPMChainGraph"


namespace
{
	bool IsComponentValid(UPPMChainGraphExecutorComponent* InPPMChainComponent, UWorld* CurrentWorld)
	{
		return InPPMChainComponent && InPPMChainComponent->GetWorld() == CurrentWorld;
	}

}

void UPPMChainGraphWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	SceneViewExtension = FSceneViewExtensions::NewExtension<FPPMChainGraphSceneViewExtension>(this);
	Super::Initialize(Collection);
}

void UPPMChainGraphWorldSubsystem::Deinitialize()
{
	// This has to be done on render thread as otherwise there could be a race condition when new level is loaded. 
	ENQUEUE_RENDER_COMMAND(ReleaseSVE)([this](FRHICommandListImmediate& RHICmdList)
	{
		// Prevent this SVE from being gathered, in case it is kept alive by a strong reference somewhere else.
		{
			SceneViewExtension->IsActiveThisFrameFunctions.Empty();

			FSceneViewExtensionIsActiveFunctor IsActiveFunctor;

			IsActiveFunctor.IsActiveFunction = [](const ISceneViewExtension* InSceneViewExtension, const FSceneViewExtensionContext& InContext)
			{
				return TOptional<bool>(false);
			};

			SceneViewExtension->IsActiveThisFrameFunctions.Add(IsActiveFunctor);
		}

		SceneViewExtension.Reset();
	});

	// Finish all rendering commands before cleaning up actors.
	FlushRenderingCommands();
	{
		FScopeLock ScopeLock(&ComponentAccessCriticalSection);
		PPMChainGraphComponents.Reset();
	}
	Super::Deinitialize();

}

void UPPMChainGraphWorldSubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	GatherActivePasses();
}

void UPPMChainGraphWorldSubsystem::AddPPMChainGraphComponent(TWeakObjectPtr<UPPMChainGraphExecutorComponent> InComponent)
{
	if (!PPMChainGraphComponents.Contains(InComponent))
	{
		FScopeLock ScopeLock(&ComponentAccessCriticalSection);
		PPMChainGraphComponents.Add(InComponent);
	}
}

void UPPMChainGraphWorldSubsystem::RemovePPMChainGraphComponent(TWeakObjectPtr<UPPMChainGraphExecutorComponent> InComponent)
{
	if (PPMChainGraphComponents.Contains(InComponent))
	{
		FScopeLock ScopeLock(&ComponentAccessCriticalSection);
		PPMChainGraphComponents.Remove(InComponent);
	}
}

void UPPMChainGraphWorldSubsystem::GatherActivePasses()
{
	TSet<uint32> TempActivePasses;
	for (const TWeakObjectPtr<UPPMChainGraphExecutorComponent>& PPMChainGraphComponent : PPMChainGraphComponents)
	{
		if (!PPMChainGraphComponent.IsValid())
		{
			continue;
		}
		for (uint32 PassId = 1; PassId <= (uint32)EPPMChainGraphExecutionLocation::AfterVisualizeDepthOfField; PassId++)
		{
			if (PPMChainGraphComponent->IsActiveDuringPass_GameThread((EPPMChainGraphExecutionLocation)(PassId)))
			{
				TempActivePasses.FindOrAdd(PassId);
			}
		}
	}
	{
		FScopeLock ScopeLock(&ActiveAccessCriticalSection);
		ActivePasses = MoveTemp(TempActivePasses);
	}
}

#undef LOCTEXT_NAMESPACE