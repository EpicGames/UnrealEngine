// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Subsystems/WorldSubsystem.h"
#include "Engine/EngineBaseTypes.h"

#if WITH_EDITOR
#include "EditorUndoClient.h"
#endif

#include "PPMChainGraphWorldSubsystem.generated.h"

class UPPMChainGraphExecutorComponent;

/**
 * World subsystem that is responsible for gathering and keeping track of PPM Chain Graph components.
 */
UCLASS()
class UPPMChainGraphWorldSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
public:

	// Subsystem Init/Deinit
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	virtual bool IsTickableInEditor() const { return true; }
	virtual void Tick(float DeltaTime) override;

	// For profiling.
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UPPMChainGraphWorldSubsystem, STATGROUP_Tickables); }
public:
	/** Adds a PPM Chain Graph component to the list of components to potentially be rendered. */
	void AddPPMChainGraphComponent(TWeakObjectPtr<UPPMChainGraphExecutorComponent> InComponent);

	/** Removes a PPM Chain Graph component. Typically this means that the component is removed from the world. */
	void RemovePPMChainGraphComponent(TWeakObjectPtr<UPPMChainGraphExecutorComponent> InComponent);
private:
	/** Populates Active passes set to be used by SVE to subscribe to the active passes. */
	void GatherActivePasses();

private:
	TSharedPtr<class FPPMChainGraphSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	FCriticalSection ComponentAccessCriticalSection;

	/** 
	* All active chain graph components currently present in associated world.
	*/
	TSet<TWeakObjectPtr<UPPMChainGraphExecutorComponent>> PPMChainGraphComponents;

	/**
	* Aggregation of active passes, so that Scene View Extension knows to which passes to subscribe. uint32 value Reflects EPPMChainGraphExecutionLocation
	*/
	TSet<uint32> ActivePasses;

	FCriticalSection ActiveAccessCriticalSection;

public:
	friend class FPPMChainGraphSceneViewExtension;
};