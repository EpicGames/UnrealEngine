// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

#include "PPMChainGraph.generated.h"

class UTextureRenderTarget2D;
class UMaterial;
class ACameraActor;

UENUM()
enum class EPPMChainGraphPPMInputId : uint8
{
	PPMInputMaping_Unassigned UMETA(DisplayName = "Unassigned"),
	/** Map to equivalent input in Scene Textures node in the material. */
	PPMInputMaping_0 UMETA(DisplayName = "PostProcessInput0"),
	PPMInputMaping_1 UMETA(DisplayName = "PostProcessInput1"),
	PPMInputMaping_2 UMETA(DisplayName = "PostProcessInput2"),
	PPMInputMaping_3 UMETA(DisplayName = "PostProcessInput3"),
	PPMInputMaping_4 UMETA(DisplayName = "PostProcessInput4"),
};


UENUM()
enum class EPPMChainGraphOutput : uint8
{
	/**
	* Write back into Scene Color.
	*/
	PPMOutput_SceneColor	UMETA(DisplayName = "Write Back into Scene Color"),

	/** 
	* Writes into a temporary render target to be used later in the chain. 
	* Users need to provide the name for the render target for future identification.
	*/
	PPMOutput_RenderTarget	UMETA(DisplayName = "Temporary Render Target"),

};


UENUM()
enum class EPPMChainGraphExecutionLocation : uint8
{
	// Default place of execution right before the rest of post processing starts.
	PrePostProcess				UMETA(DisplayName = "Before Post Processing"),

	// SSR Input isn't supported. 
	//AfterSSRInput				UMETA(DisplayName = "After SSR Input"),
	// 
	// This is eventually mapped to equivalent execution location enum: ISceneViewExtension::EPostProcessingPass.
	AfterMotionBlur = 2			UMETA(DisplayName = "After Motion Blur"),
	AfterToneMap				UMETA(DisplayName = "After Tonemap"),
	AfterFXAA					UMETA(DisplayName = "After FXAA"),
	AfterVisualizeDepthOfField	UMETA(DisplayName = "After Visualize Depth of Field"),
};


/**
* This struct is used for customizing Input and External Texture input selection.
*/
USTRUCT(BlueprintType)
struct PPMCHAINGRAPH_API FPPMChainGraphInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Input)
	FString InputId;
};


USTRUCT(Blueprintable)
struct PPMCHAINGRAPH_API FPPMChainGraphPostProcessPass
{
	GENERATED_USTRUCT_BODY()

	/** Is pass enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Pass Settings")
	bool bEnabled = true;

	/** Inputs from previous passes. Map this to Scene Texture node in Post Process Material. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Pass Inputs")
	TMap<EPPMChainGraphPPMInputId, FPPMChainGraphInput> Inputs;

	/**
	* Which material should be executed during this pass.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Pass Material")
	TObjectPtr<UMaterial> PostProcessMaterial;

	/** 
	* Where should this pass write to. By selecting Temporary Render Target as an option, 
	* Users can avoid writing directly into Scene Color. For example this pass can operate
	* on textures other than Scene Color to be used in one of the later passes.
	* Keep in mind that the final pass will always write back into Scene Color.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Pass Output")
	EPPMChainGraphOutput Output = EPPMChainGraphOutput::PPMOutput_SceneColor;

	/** Use this to identify the Output of the current pass to be used later. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Pass Output", meta = (EditCondition = "Output == EPPMChainGraphOutput::PPMOutput_RenderTarget", EditConditionHides))
	FString TemporaryRenderTargetId;
};


/**
* Post Process Material Chain Graph defines a collection of Post Process Material of passes that are executed one after another.
* Individual passes can write into Scene Color or Temporary render target, but at the end of chain graph the combined result is 
* always written into Scene Color.
*/
UCLASS(BlueprintType, meta = (DisplayName = "Post Process Material Chain Graph"))
class PPMCHAINGRAPH_API UPPMChainGraph : public UObject
{
	GENERATED_BODY()

public:

	UPPMChainGraph(const FObjectInitializer& ObjectInitializer);
public:

	/** 
	* Identifies at which point of post processing this graph is going to be executed. 
	* These are different to the setting in Post Process Material.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Graph Execution Settings")
	EPPMChainGraphExecutionLocation PointOfExecution;

	/**
	* External textures and rendertargets that can be mapped to Scene Texture Inputs.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "External Inputs")
	TMap<FString, TObjectPtr<UTexture2D>> ExternalTextures;

	/**
	* Post Process Material Passes. Each pass can write into the Scene Color or into a temporary render target that can
	* be referenced in subsequent passes. At the end of all passes the result is always written into Scene Color.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Passes", meta = (TitleProperty = "TemporaryRenderTargetId"))
	TArray<FPPMChainGraphPostProcessPass> Passes;
};