// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "PPMChainGraphActor.generated.h"

class UPPMChainGraphExecutorComponent;

UCLASS(Blueprintable, meta = (DisplayName = "Post Process Material Chain Graph Actor"))
class PPMCHAINGRAPH_API APPMChainGraphActor : public AActor
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(Category = MediaPlate, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<UPPMChainGraphExecutorComponent> PPMChainGraphExecutorComponent;

#if WITH_EDITORONLY_DATA

	/** Billboard component for this actor. */
	UPROPERTY(Transient)
	TObjectPtr<class UBillboardComponent> SpriteComponent;

#endif // WITH_EDITORONLY_DATA

};