// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PPMChainGraph.h"
#include "Components/SceneComponent.h"

#include "PPMChainGraphComponent.generated.h"

class UCameraComponent;
class UPPMChainGraphWorldSubsystem;

DECLARE_LOG_CATEGORY_EXTERN(LogPPMChainGraph, Log, All);

struct FPPMChainGraphProxy
{
	TObjectPtr<ACameraActor> CameraOwner;

	EPPMChainGraphExecutionLocation PointOfExecution;
	TMap<FString, TObjectPtr<UTexture2D>> ExternalTextures;
	TArray<TSharedPtr<FPPMChainGraphPostProcessPass>> Passes;
};

UENUM()
enum class ECameraViewHandling : uint8
{
	IgnoreCameraViews UMETA(DisplayName = "Ignore Selected Camera Views"),
	RenderOnlyInSelectedCameraViews UMETA(DisplayName = "Render Only In Selected Camera Views"),
};

UCLASS(hidecategories = (Transform, Collision, Object, Physics, SceneComponent, PostProcessVolume, Projection, Rendering, PlanarReflection), ClassGroup = Rendering, editinlinenew, meta = (BlueprintSpawnableComponent, DisplayName = "Post Process Material Chain Graph Executor Component"))
class PPMCHAINGRAPH_API UPPMChainGraphExecutorComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()
public:
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual void BeginDestroy() override;

public:

	/** Get a copy of Render Proxies for rendering graphs. */
	TArray<TSharedPtr<FPPMChainGraphProxy>> GetChainGraphRenderProxies(EPPMChainGraphExecutionLocation InPointOfExecution);

	/** Identifies if this component needs to render during the specified pass. */
	bool IsActiveDuringPass_GameThread(EPPMChainGraphExecutionLocation InPointOfExecution);

public:
	/** PPM Chain Graph Component can either exclude selected cameras from being rendered or render only in selected camera views. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Camera View Settings")
	ECameraViewHandling CameraViewHandlingMode = ECameraViewHandling::IgnoreCameraViews;

	/** 
	* Depending on the selection in the option above the following cameras will either:
	* 1. Be ignored and will not get PPM Chain Graph rendered into. 
	* 2. Or rendered only in selected camera views.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Camera View Settings")
	TArray<TSoftObjectPtr<ACameraActor>> CameraList;

	/** A list of graphs to be executed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Post Process Material Chain Graphs")
	TArray<TObjectPtr<UPPMChainGraph>> PPMChainGraphs;

private:
	/** Transfers the render state on game thread. */
	void TransferState();

	/** Process aggregated warnings and notify the user. */
	void ProcessWarnings();

private:
	// Critical section for tranfering state for rendering.
	FCriticalSection StateTransferCriticalSection;

	// Reference to the subsystem to add or remove this component to the list of Graphs to be considered.
	TWeakObjectPtr<UPPMChainGraphWorldSubsystem> PPMChainGraphSubsystem;

	// Populated on Game thread and used on Render thread for thread safety. Stored against the location of execution.
	TMap<EPPMChainGraphExecutionLocation, TArray<TSharedPtr<FPPMChainGraphProxy>>> PPMChainGraphsRenderProxies;

	// Collects warnings notifying users of invalid data in graphs or issues with ComponentAttachement.
	TSharedPtr<void> AggregatedWarnings;
};