// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PPMChainGraph.h"

#include "PPMChainGraph.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_PPMChainGraph"


FText UAssetDefinition_PPMChainGraph::GetAssetDescription(const struct FAssetData& AssetData) const
{
	return FText::FromString("Post Process Material Chain Graph");
}


TSoftClassPtr<UObject> UAssetDefinition_PPMChainGraph::GetAssetClass() const
{
	return UPPMChainGraph::StaticClass();
}

FText UAssetDefinition_PPMChainGraph::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDefinition_PPMChainGraph", "Post Process Material Chain Graph");
}

FLinearColor UAssetDefinition_PPMChainGraph::GetAssetColor() const
{
	return FLinearColor::White;
}


#undef LOCTEXT_NAMESPACE
