// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PPMChainGraph.generated.h"

UCLASS()
class UAssetDefinition_PPMChainGraph : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// Begin UAssetDefinition interface
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}
	virtual bool CanImport() const override { return false; }
	
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	// End UAssetDefinition interface
};
