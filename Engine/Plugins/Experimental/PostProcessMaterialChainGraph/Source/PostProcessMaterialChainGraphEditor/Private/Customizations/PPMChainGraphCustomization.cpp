// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphCustomization.h"

#include "PPMChainGraph.h"
#include "PPMChainGraphInputCustomization.h"
#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "PPMChainGraphCustomization"

void FPPMChainGraphCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	TWeakObjectPtr<UPPMChainGraph> ChainGraph = Cast<UPPMChainGraph>(Objects[0].Get());

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FPPMChainGraphInput::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([ChainGraph] { return FPPMChainGraphInputCustomization::MakeInstance(ChainGraph); }));
}

#undef LOCTEXT_NAMESPACE
