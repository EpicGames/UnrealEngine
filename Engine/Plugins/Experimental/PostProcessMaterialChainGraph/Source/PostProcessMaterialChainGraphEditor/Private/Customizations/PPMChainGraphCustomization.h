// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"


class FPPMChainGraphCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FPPMChainGraphCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

