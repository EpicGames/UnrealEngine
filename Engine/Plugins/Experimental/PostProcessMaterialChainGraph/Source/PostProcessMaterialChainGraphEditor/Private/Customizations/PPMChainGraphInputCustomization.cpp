// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphInputCustomization.h"

#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "PPMChainGraphInputCustomization"

FPPMChainGraphInputCustomization::FPPMChainGraphInputCustomization(TWeakObjectPtr<UPPMChainGraph> InPPMChainGraph)
	: PPMChainGraphPtr(MoveTemp(InPPMChainGraph))
{
}

void FPPMChainGraphInputCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CachedInputsProperty = InPropertyHandle;

	TSharedPtr<IPropertyHandle> InputsArrayProperty = InPropertyHandle->GetParentHandle();

	CachedPassProperty = InputsArrayProperty->GetParentHandle();

	if (InPropertyHandle->GetNumPerObjectValues() == 1 && InPropertyHandle->IsValidHandle())
	{
		FProperty* Property = InPropertyHandle->GetProperty();
		check(Property && CastField<FStructProperty>(Property) && CastField<FStructProperty>(Property)->Struct && CastField<FStructProperty>(Property)->Struct->IsChildOf(FPPMChainGraphInput::StaticStruct()));

		TArray<void*> RawData;
		InPropertyHandle->AccessRawData(RawData);

		check(RawData.Num() == 1);
		FPPMChainGraphInput* InputValue = reinterpret_cast<FPPMChainGraphInput*>(RawData[0]);

		check(InputValue);
		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

		HeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(512)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(MakeAttributeLambda([InputValue] {
						const FString InputId = InputValue->InputId;
							
						if(!InputId.IsEmpty())
						{
							return FText::FromString(InputId);
						}

						return LOCTEXT("NotSelected", "<Not Selected>");
					}))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FPPMChainGraphInputCustomization::PopulateComboBox)
					.ContentPadding(FMargin(4.0, 2.0))
				]
			].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
	}
}

void FPPMChainGraphInputCustomization::AddComboBoxEntry(FMenuBuilder& InMenuBuilder, const FPPMChainGraphInput& InTextureInput)
{
	InMenuBuilder.AddMenuEntry(
		FText::FromString(InTextureInput.InputId),
		FText::FromString(InTextureInput.InputId),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, InTextureInput]
			{
				if (FStructProperty* StructProperty = CastField<FStructProperty>(CachedInputsProperty->GetProperty()))
				{
					check(true);
					TArray<void*> RawData;
					CachedInputsProperty->AccessRawData(RawData);
					FPPMChainGraphInput* TextureInput = reinterpret_cast<FPPMChainGraphInput*>(RawData[0]);

					FString TextValue;
					StructProperty->Struct->ExportText(TextValue, &InTextureInput, TextureInput, nullptr, EPropertyPortFlags::PPF_None, nullptr);
					ensure(CachedInputsProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
				}
			}
			),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, InTextureInput]
			{
				return true;
			})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
		);
}

TSharedRef<SWidget> FPPMChainGraphInputCustomization::PopulateComboBox()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	TArray<FString> ExistingSubMenus;

	MenuBuilder.BeginSection("AllPossibleInputs", LOCTEXT("InputsSelection", "Texture Inputs"));
	{
		TArray<void*> RawData;
		CachedPassProperty->AccessRawData(RawData);
		FPPMChainGraphPostProcessPass* CurrentPass = reinterpret_cast<FPPMChainGraphPostProcessPass*>(RawData[0]);

		if (PPMChainGraphPtr.IsValid())
		{
			TArray<FString> InputIds;
			for (TTuple<FString, TObjectPtr<UTexture2D>> ExternalTexture : PPMChainGraphPtr->ExternalTextures)
			{
				InputIds.Add(ExternalTexture.Key);
				const FPPMChainGraphInput GraphInputOption = { ExternalTexture.Key };
				
				AddComboBoxEntry(MenuBuilder, GraphInputOption);
			}

			for (const FPPMChainGraphPostProcessPass& Pass : PPMChainGraphPtr->Passes)
			{
				if (Pass.TemporaryRenderTargetId.IsEmpty())
				{
					continue;
				}

				if (InputIds.Contains(Pass.TemporaryRenderTargetId))
				{
					continue;
				}

				// We collect passes up to the current one and ignore anything that comes after.
				if (CurrentPass == &Pass)
				{
					break;
				}

				InputIds.Add(Pass.TemporaryRenderTargetId);
				const FPPMChainGraphInput GraphInputOption = { Pass.TemporaryRenderTargetId };
				AddComboBoxEntry(MenuBuilder, GraphInputOption);
			}

			if (InputIds.Num() <= 0)
			{
				MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoValidInputs", "No valid inputs found"), false, false);
			}
		}
		else
		{
			MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("InvalidChainGraph", "Invalid PPM Chain Graph object selected"), false, false);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
