// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "PPMChainGraph.h"
#include "Widgets/SWidget.h"

class UPPMChainGraph;

class FPPMChainGraphInputCustomization : public IPropertyTypeCustomization
{
public:
	FPPMChainGraphInputCustomization(TWeakObjectPtr<UPPMChainGraph> InConfigurationObjectProperty);

	/** IPropertyTypeCustomization interface stub. */
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override {}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakObjectPtr<UPPMChainGraph> InConfigurationObjectProperty)
	{
		return MakeShared<FPPMChainGraphInputCustomization>(MoveTemp(InConfigurationObjectProperty));
	}

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

protected:

	TSharedPtr<IPropertyHandle> CachedInputsProperty;
	TSharedPtr<IPropertyHandle> CachedPassProperty;

	TWeakObjectPtr<UPPMChainGraph> PPMChainGraphPtr;

private:
	void AddComboBoxEntry(FMenuBuilder& InMenuBuilder, const FPPMChainGraphInput& InTextureInput);

	TSharedRef<SWidget> PopulateComboBox();
};


