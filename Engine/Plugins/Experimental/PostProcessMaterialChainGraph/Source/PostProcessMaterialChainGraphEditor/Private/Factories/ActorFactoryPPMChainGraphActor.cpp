// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryPPMChainGraphActor.h"
#include "PPMChainGraph.h"
#include "PPMChainGraphActor.h"
#include "PPMChainGraphComponent.h"

#include "Async/Async.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryPPMChainGraphActor)

#define LOCTEXT_NAMESPACE "ActorFactoryPPMChainGraphActor"

UActorFactoryPPMChainGraphActor::UActorFactoryPPMChainGraphActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PPMChainGraphActorDisplayName", "Post Process Material Chain Graph Actor");
	NewActorClass = APPMChainGraphActor::StaticClass();
}

bool UActorFactoryPPMChainGraphActor::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid())
	{
		UClass* AssetClass = AssetData.GetClass();
		if ((AssetClass != nullptr) && (AssetClass->IsChildOf(UPPMChainGraph::StaticClass())))
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		return true;
	}
}

void UActorFactoryPPMChainGraphActor::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	SetUpActor(Asset, NewActor);
}

void UActorFactoryPPMChainGraphActor::SetUpActor(UObject* Asset, AActor* Actor)
{
	if (Actor != nullptr)
	{
		APPMChainGraphActor* ChainGraphActor = CastChecked<APPMChainGraphActor>(Actor);

		UPPMChainGraph* ChainGraph = Cast<UPPMChainGraph>(Asset);

		if ((ChainGraph != nullptr) && (ChainGraphActor->PPMChainGraphExecutorComponent != nullptr) && !ChainGraphActor->bIsEditorPreviewActor)
		{
			if (UPPMChainGraphExecutorComponent* PPMChainGraphComponent = GetValid(ChainGraphActor->PPMChainGraphExecutorComponent))
			{
				if (!PPMChainGraphComponent->PPMChainGraphs.Contains(ChainGraph))
				{
					PPMChainGraphComponent->PPMChainGraphs.Add(ChainGraph);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

