// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryPPMChainGraphActor.generated.h"

UCLASS()
class UActorFactoryPPMChainGraphActor : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory Interface

private:
	/**
	 * Set up a PPM Chain Graph Actor.
	 */
	void SetUpActor(UObject* Asset, AActor* Actor);
};
