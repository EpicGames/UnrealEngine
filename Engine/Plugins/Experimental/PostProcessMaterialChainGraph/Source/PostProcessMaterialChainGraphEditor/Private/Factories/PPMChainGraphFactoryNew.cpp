// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphFactoryNew.h"

#include "AssetTypeCategories.h"
#include "PPMChainGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PPMChainGraphFactoryNew)


UPPMChainGraphFactoryNew::UPPMChainGraphFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPPMChainGraph::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


UObject* UPPMChainGraphFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPPMChainGraph>(InParent, InClass, InName, Flags);
}


uint32 UPPMChainGraphFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}


bool UPPMChainGraphFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

