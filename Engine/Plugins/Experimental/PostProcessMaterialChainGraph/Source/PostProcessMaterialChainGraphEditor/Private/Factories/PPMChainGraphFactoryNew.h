// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "PPMChainGraphFactoryNew.generated.h"


UCLASS(hidecategories=Object)
class UPPMChainGraphFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
