// Copyright Epic Games, Inc. All Rights Reserved.

#include "PPMChainGraphEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Customizations/PPMChainGraphCustomization.h"
#include "PropertyEditorModule.h"
#include "PPMChainGraph.h"

void FPPMChainGraphEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UPPMChainGraph::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPPMChainGraphCustomization::MakeInstance));
}

void FPPMChainGraphEditorModule::ShutdownModule()
{
	
}

IMPLEMENT_MODULE(FPPMChainGraphEditorModule, PPMChainGraphEditor);
DEFINE_LOG_CATEGORY(LogPPMChainGraphEditor);
