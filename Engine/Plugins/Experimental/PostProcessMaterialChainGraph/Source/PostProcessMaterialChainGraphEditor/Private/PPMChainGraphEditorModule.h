// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPPMChainGraphEditorModule.h"
#include "CoreMinimal.h"


class FPPMChainGraphEditorModule : public IPPMChainGraphEditorModule, public TSharedFromThis<FPPMChainGraphEditorModule>
{
public:

	virtual ~FPPMChainGraphEditorModule() = default;

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};

