// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"


DECLARE_LOG_CATEGORY_EXTERN(LogPPMChainGraphEditor, Log, All);

class IPPMChainGraphEditorModule : public IModuleInterface
{
public:
};

